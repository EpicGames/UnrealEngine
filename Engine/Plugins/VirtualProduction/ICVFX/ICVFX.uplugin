{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "ICVFX",
	"Description": "Conveniently collects plugins for In-Camera VFX",
	"Category": "Virtual Production",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"CanContainContent": false,
	"IsBetaVersion": true,
	"Installed": false,
	"Plugins": [
		{
			"Name": "AjaMedia",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64"
			]
		},
		{
			"Name": "BlackmagicMedia",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "CameraCalibration",
			"Enabled": true
		},
		{
			"Name": "Composure",
			"Enabled": true
		},
		{
			"Name": "ColorCorrectRegions",
			"Enabled": true
		},
		{
			"Name": "GPULightmass",
			"Enabled": true
		},
		{
			"Name": "LevelSnapshots",
			"Enabled": true
		},
		{
			"Name": "MultiUserClient",
			"Enabled": true
		},
		{
			"Name": "LiveLink",
			"Enabled": true
		},
		{
			"Name": "LiveLinkOverNDisplay",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "LiveLinkLens",
			"Enabled": true
		},
		{
			"Name": "LiveLinkCamera",
			"Enabled": true
		},
		{
			"Name": "MediaFrameworkUtilities",
			"Enabled": true
		},
		{
			"Name": "MultiUserTakes",
			"Enabled": true
		},
		{
			"Name": "nDisplay",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64",
				"Linux"
			]
		},
		{
			"Name": "OSC",
			"Enabled": true
		},
		{
			"Name": "OpenColorIO",
			"Enabled": true
		},
		{
			"Name": "RemoteControl",
			"Enabled": true
		},
		{
			"Name": "RemoteControlWebInterface",
			"Enabled": true
		},
		{
			"Name": "StageMonitoring",
			"Enabled": true
		},
		{
			"Name": "Switchboard",
			"Enabled": true
		},
		{
			"Name": "Takes",
			"Enabled": true
		},
		{
			"Name": "TimedDataMonitor",
			"Enabled": true
		},
		{
			"Name": "SequencerScripting",
			"Enabled": true
		},
		{
			"Name": "ConsoleVariables",
			"Enabled": true
		},
		{
			"Name": "VirtualProductionUtilities",
			"Enabled": true
		},
		{
			"Name": "EpicStageApp",
			"Enabled": true
		},
		{
			"Name": "RivermaxMedia",
			"Enabled": true,
			"SupportedTargetPlatforms": [
				"Win64"
			]
		},
		{
			"Name": "ICVFXTesting",
			"Enabled": true
		}
	]
}
